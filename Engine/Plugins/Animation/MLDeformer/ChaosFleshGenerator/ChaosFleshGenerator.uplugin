{
	"FileVersion": 3,
	"Version": 0,
	"VersionName": "0.0",
	"FriendlyName": "Chaos Flesh Generator",
	"Description": "Chaos Flesh Data Generator for ML Deformer",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "https://forums.unrealengine.com/",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosFleshGenerator",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ChaosFlesh",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		},
		{
			"Name": "MLDeformerFramework",
			"Enabled": true
		}
	]
}