// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFleshGenerator.h"
#include "ChaosFleshGeneratorPrivate.h"
#include "ChaosFleshGeneratorSimulation.h"
#include "ChaosFleshGeneratorThreading.h"
#include "Engine/SkeletalMesh.h"
#include "FleshGeneratorProperties.h"
#include "Misc/AsyncTaskNotification.h"
#include "Dataflow/DataflowSimulationGeometryCache.h"

DEFINE_LOG_CATEGORY(LogChaosFleshGenerator);

#define LOCTEXT_NAMESPACE "ChaosFleshGenerator"

namespace UE::Chaos::FleshGenerator
{
	FChaosFleshGenerator::FChaosFleshGenerator()
	{
		Properties = TObjectPtr<UFleshGeneratorProperties>(NewObject<UFleshGeneratorProperties>());
	}

	FChaosFleshGenerator::~FChaosFleshGenerator()
	{
		if (TaskResource != nullptr)
		{
			TaskResource->FreeSimResources_GameThread();
		}
	}

	void FChaosFleshGenerator::Tick(float DeltaTime)
	{
		if (PendingAction == EFleshGeneratorActions::StartGenerate)
		{
			StartGenerate();
		}
		else if (PendingAction == EFleshGeneratorActions::TickGenerate)
		{
			TickGenerate();
		}
	}

	TStatId FChaosFleshGenerator::GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FChaosFleshGenerator, STATGROUP_Tickables);
	}

	void FChaosFleshGenerator::StartGenerate()
	{
		check(PendingAction == EFleshGeneratorActions::StartGenerate);
		if (Properties->FleshAsset == nullptr)
		{
			UE_LOGF(LogChaosFleshGenerator, Error, "FleshAsset is null.");
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}
		if (Properties->SkeletalMeshAsset == nullptr)
		{
			UE_LOGF(LogChaosFleshGenerator, Error, "SkeletalMeshAsset is null.");
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}
		if (Properties->SkeletalMeshAsset->GetSkeleton() == nullptr)
		{
			UE_LOGF(LogChaosFleshGenerator, Error, "SkeletalMeshAssets skeleton is null.");
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}
		if (Properties->FleshAsset->SkeletalMesh != Properties->SkeletalMeshAsset)
		{
			UE_LOGF(LogChaosFleshGenerator, Error, "Flesh assets skeletal mesh is not the same as the Generators");
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}
		if (Properties->AnimationSequence == nullptr)
		{
			UE_LOGF(LogChaosFleshGenerator, Error, "AnimationSequence is null.");
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}
		UGeometryCache* const Cache = GetCache();
		if (Cache == nullptr)
		{
			UE_LOGF(LogChaosFleshGenerator, Error, "Cannot find or create geometry cache.");
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}
		if (TaskResource != nullptr)
		{
			UE_LOGF(LogChaosFleshGenerator, Error, "Previous generation is still running.");
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}

		using UE::Chaos::FleshGenerator::Private::GetMeshImportVertexMap;
		TOptional<TArray<int32>> OptionalMap = GetMeshImportVertexMap(*Properties->SkeletalMeshAsset, *Properties->FleshAsset);
		if (!OptionalMap)
		{
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}
		TaskResource = TSharedPtr<FTaskResource>(new FTaskResource);

		using Private::ParseFrames;
		using Private::Range;

		// add back debugging support
		TaskResource->FramesToSimulate = Range(Properties->AnimationSequence->GetNumberOfSampledKeys());
		if (Properties->FramesToSimulate.Len() > 0)
		{
			TaskResource->FramesToSimulate = ParseFrames(Properties->FramesToSimulate);
		}


		const int32 NumFrames = TaskResource->FramesToSimulate.Num();
		if (NumFrames == 0)
		{
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}
		TaskResource->SimulatedPositions.SetNum( NumFrames);

		if (!TaskResource->AllocateSimResources_GameThread(Properties, 1 /*NumThreads*/))
		{
			PendingAction = EFleshGeneratorActions::NoAction;
			return;
		}
		TaskResource->Cache = Cache;

		TUniquePtr<FLaunchSimsTask> Task = MakeUnique<FLaunchSimsTask>(TaskResource, Properties);
		TaskResource->Executer = MakeUnique<FTaskResource::FExecuterType>(MoveTemp(Task));
		TaskResource->Executer->StartBackgroundTask();
	
		FAsyncTaskNotificationConfig NotificationConfig;
		NotificationConfig.TitleText = LOCTEXT("SimulateFlesh", "Simulating Flesh");
		NotificationConfig.ProgressText = FText::FromString(TEXT("0%"));
		NotificationConfig.bCanCancel = true;
		NotificationConfig.bKeepOpenOnSuccess = true;
		NotificationConfig.bKeepOpenOnFailure = true;
		TaskResource->Notification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);
		TaskResource->StartTime = FDateTime::UtcNow();
		TaskResource->LastUpdateTime = TaskResource->StartTime;
	
		const TArray<int32>& Map = OptionalMap.GetValue();
		TaskResource->ImportedVertexNumbers = TArray<uint32>(reinterpret_cast<const uint32*>(Map.GetData()), Map.Num());
		PendingAction = EFleshGeneratorActions::TickGenerate;
	}
	
	void FChaosFleshGenerator::TickGenerate()
	{
		check(PendingAction == EFleshGeneratorActions::TickGenerate && TaskResource != nullptr);
			
		bool bFinished = false;
		const bool bCancelled = TaskResource->Notification->GetPromptAction() == EAsyncTaskNotificationPromptAction::Cancel;
		if (TaskResource->Executer->IsDone())
		{
			bFinished = true;
		}
		else if (bCancelled)
		{
			TaskResource->Cancel();
			bFinished = true;
		}
			
		if (!bFinished)
		{
			TaskResource->FlushRendering();
			const FDateTime CurrentTime = FDateTime::UtcNow();
			const double SinceLastUpdate = (CurrentTime - TaskResource->LastUpdateTime).GetTotalSeconds();
			if (SinceLastUpdate < 0.2)
			{
				return;
			}
			
			const int32 NumSimulatedFrames = TaskResource->NumSimulatedFrames.load();
			const int32 NumTotalFrames = TaskResource->FramesToSimulate.Num();
			const FText ProgressMessage = FText::FromString(FString::Printf(TEXT("Finished %d/%d, %.1f%%"), NumSimulatedFrames, NumTotalFrames, 100.0 * NumSimulatedFrames / NumTotalFrames));
			TaskResource->Notification->SetProgressText(ProgressMessage);
			TaskResource->LastUpdateTime = CurrentTime;
		}
		else
		{
			FreeTaskResource(bCancelled);
			PendingAction = EFleshGeneratorActions::NoAction;
		}
	}

	UFleshGeneratorProperties& FChaosFleshGenerator::GetProperties() const
	{
		return *Properties;
	}
	
	void FChaosFleshGenerator::RequestAction(EFleshGeneratorActions ActionType)
	{
		if (PendingAction != EFleshGeneratorActions::NoAction)
		{
			return;
		}
		PendingAction = ActionType;
	}
	
	
	UGeometryCache* FChaosFleshGenerator::GetCache() const
	{
		return Properties->SimulatedCache;
	}
	
	void FChaosFleshGenerator::FreeTaskResource(bool bCancelled)
	{
		TaskResource->Notification->SetProgressText(LOCTEXT("Finishing", "Finishing, please wait"));
		TaskResource->FreeSimResources_GameThread();
		const FDateTime CurrentTime = FDateTime::UtcNow();
		UE_LOGF(LogChaosFleshGenerator, Log, "Training finished in %f seconds", (CurrentTime - TaskResource->StartTime).GetTotalSeconds());
	
		{
			UE::Chaos::FleshGenerator::Private::FTimeScope TimeScope(TEXT("Saving"));
			UE::DataflowSimulationGeometryCache::SaveGeometryCache(*TaskResource->Cache, Properties->SolverTiming.FrameRate, *Properties->SkeletalMeshAsset, TaskResource->ImportedVertexNumbers, TaskResource->SimulatedPositions);
			UE::DataflowSimulationGeometryCache::SavePackage(*TaskResource->Cache);
		}
		if (bCancelled)
		{
			TaskResource->Notification->SetProgressText(LOCTEXT("Cancelled", "Cancelled"));
			TaskResource->Notification->SetComplete(false);
		}
		else
		{
			TaskResource->Notification->SetProgressText(LOCTEXT("Finished", "Finished"));
			TaskResource->Notification->SetComplete(true);
		}
		TaskResource.Reset();
	}
};

#undef LOCTEXT_NAMESPACE
