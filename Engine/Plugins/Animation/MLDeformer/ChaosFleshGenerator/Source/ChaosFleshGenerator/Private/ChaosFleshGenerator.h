// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectPtr.h"
#include "TickableEditorObject.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogChaosFleshGenerator, Log, All);

class UGeometryCache;
class UFleshGeneratorProperties;

namespace UE::Chaos::FleshGenerator
{
	struct FTaskResource;

	enum class EFleshGeneratorActions
	{
		NoAction,
		StartGenerate,
		TickGenerate
	};

	class FChaosFleshGenerator : public FTickableEditorObject
	{
	public:
		FChaosFleshGenerator();
		virtual ~FChaosFleshGenerator();

		//~ Begin FTickableEditorObject Interface
		virtual void Tick(float DeltaTime) override;
		virtual TStatId GetStatId() const override;
		virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
		//~ End FTickableEditorObject Interface

		UFleshGeneratorProperties& GetProperties() const;
		void RequestAction(EFleshGeneratorActions Action);
	private:

		void StartGenerate();
		void TickGenerate();
		void FreeTaskResource(bool bCancelled);
		UGeometryCache* GetCache() const;

		TObjectPtr<UFleshGeneratorProperties> Properties;
		EFleshGeneratorActions PendingAction = EFleshGeneratorActions::NoAction;
		TSharedPtr<FTaskResource> TaskResource;
	};
};