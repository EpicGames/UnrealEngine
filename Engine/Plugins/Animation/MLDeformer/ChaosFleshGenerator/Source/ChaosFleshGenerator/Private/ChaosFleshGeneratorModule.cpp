// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosFleshGeneratorToolsMenuExtender.h"
#include "MLDeformerEditorToolkit.h"
#include "PropertyEditorModule.h"
#include "SFleshGeneratorWidget.h"
#include "Modules/ModuleManager.h"

namespace UE::Chaos::FleshGenerator
{
	class FChaosFleshGeneratorModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override
		{
			UE::MLDeformer::FMLDeformerEditorToolkit::AddToolsMenuExtender(CreateToolsMenuExtender());
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.RegisterCustomClassLayout("FleshGeneratorProperties", FOnGetDetailCustomizationInstance::CreateStatic(&FFleshGeneratorDetails::MakeInstance));
		}

		virtual void ShutdownModule() override
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyModule.UnregisterCustomClassLayout("FleshGeneratorProperties");
		}
	};
};

IMPLEMENT_MODULE(UE::Chaos::FleshGenerator::FChaosFleshGeneratorModule, ChaosFleshGenerator)