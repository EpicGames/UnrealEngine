// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFleshGeneratorPrivate.h"

#include "Chaos/Vector.h"
#include "ChaosFlesh/FleshCollectionEngineUtility.h"
#include "FleshGeneratorComponent.h"
#include "FileHelpers.h"
#include "GeometryCollection/Facades/CollectionTetrahedralBindingsFacade.h"
#include "GeometryCache.h"
#include "GeometryCacheCodecV1.h"
#include "GeometryCacheMeshData.h"
#include "GeometryCacheTrackStreamable.h"
#include "GeometryCacheConstantTopologyWriter.h"
#include "Internationalization/Regex.h"
#include "Logging/LogMacros.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
DEFINE_LOG_CATEGORY(LogChaosFleshGeneratorPrivate);

#define LOCTEXT_NAMESPACE "ChaosFleshGeneratorPrivate"

namespace UE::Chaos::FleshGenerator
{
	namespace Private
	{
		FTimeScope::FTimeScope(FString InName)
			: Name(MoveTemp(InName))
			, StartTime(FDateTime::UtcNow())
		{
		}

		FTimeScope::~FTimeScope()
		{
			const FTimespan Duration = FDateTime::UtcNow() - StartTime;
			UE_LOGF(LogChaosFleshGeneratorPrivate, Log, "%ls took %f secs", *Name, Duration.GetTotalSeconds());
		}

		TArray<int32> ParseFrames(const FString& FramesString)
		{
			TArray<int32> Result;
			static const FRegexPattern AllowedCharsPattern(TEXT("^[-,0-9\\s]+$"));
		
			if (!FRegexMatcher(AllowedCharsPattern, FramesString).FindNext())
			{
			    UE_LOGF(LogChaosFleshGeneratorPrivate, Error, "Input contains invalid characters.");
			    return Result;
			}
		
			static const FRegexPattern SingleNumberPattern(TEXT("^\\s*(\\d+)\\s*$"));
			static const FRegexPattern RangePattern(TEXT("^\\s*(\\d+)\\s*-\\s*(\\d+)\\s*$"));
		
			TArray<FString> Segments;
		    FramesString.ParseIntoArray(Segments, TEXT(","), true);
		    for (const FString& Segment : Segments)
		    {
		    	bool bSegmentValid = false;
		
		    	FRegexMatcher SingleNumberMatcher(SingleNumberPattern, Segment);
		    	if (SingleNumberMatcher.FindNext())
		    	{
		    	    const int32 SingleNumber = FCString::Atoi(*SingleNumberMatcher.GetCaptureGroup(1));
		    	    Result.Add(SingleNumber);
		    	    bSegmentValid = true;
		    	}
		    	else
		    	{
		    		FRegexMatcher RangeMatcher(RangePattern, Segment);
		    		if (RangeMatcher.FindNext())
		    		{
		    		    const int32 RangeStart = FCString::Atoi(*RangeMatcher.GetCaptureGroup(1));
		    		    const int32 RangeEnd = FCString::Atoi(*RangeMatcher.GetCaptureGroup(2));
		
		    		    for (int32 i = RangeStart; i <= RangeEnd; ++i)
		    		    {
		    		        Result.Add(i);
		    		    }
		    		    bSegmentValid = true;
		    		}
		    	}
		    	
		    	if (!bSegmentValid)
		    	{
		    	    UE_LOGF(LogChaosFleshGeneratorPrivate, Error, "Invalid format in segment: %ls", *Segment);
		    	}
		    }
		
			return Result;
		}
		
		TArray<int32> Range(int32 End)
		{
			TArray<int32> Result;
			Result.Reserve(End);
			for (int32 Index = 0; Index < End; ++Index)
			{
				Result.Add(Index);
			}
			return Result;
		}
		
		TOptional<TArray<int32>> GetMeshImportVertexMap(const USkinnedAsset& SkinnedMeshAsset, const UFleshAsset& FleshAsset)
		{
			constexpr int32 LODIndex = 0;
			const TOptional<TArray<int32>> None;
			const FSkeletalMeshModel* const MLDModel = SkinnedMeshAsset.GetImportedModel();
			if (!MLDModel || !MLDModel->LODModels.IsValidIndex(LODIndex))
			{
				return None;
			}
			const FSkeletalMeshLODModel& MLDLOD = MLDModel->LODModels[LODIndex];
			const TArray<int32>& Map = MLDLOD.MeshToImportVertexMap;
			if (Map.IsEmpty())
			{
				UE_LOGF(LogChaosFleshGeneratorPrivate, Warning, "MeshToImportVertexMap is empty. MLDeformer Asset should be an imported SkeletalMesh (e.g. from fbx).");
				return None;
			}

			//
			// @todo(flesh LOD) : Add support for managing vertex mappings between skeletal LOD.
			//		The cloth asset will extract the LOD from the ManagedArrayCollection.
			//		The FleshGenerator will need to do the same when Flesh supports LODS. 
			//

			TArray<FVector3f> Positions;

			const USkeletalMesh* SkeletalMeshAsset = Cast<USkeletalMesh>(&SkinnedMeshAsset);
			TSharedPtr<const FFleshCollection> FleshCollection = FleshAsset.GetFleshCollection();
			const TManagedArray<FVector3f>* RestVertices = FleshAsset.FindPositions();
			if (SkeletalMeshAsset && FleshCollection && RestVertices)
			{
				ChaosFlesh::BoundSurfacePositions(SkeletalMeshAsset, FleshCollection.Get(), RestVertices, RestVertices, Positions);

				//@todo(Flesh Sections) : Add checks for multiple sections. 
				int32 NumSections = MLDLOD.Sections.Num();
				if (NumSections != 1)
				{
					UE_LOGF(LogChaosFleshGeneratorPrivate, Warning, "SkeletalMeshAsset should have only one section.");
					return None;
				}

				for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
				{
					const FSkelMeshSection& MLDSection = MLDLOD.Sections[SectionIndex];
					if (MLDSection.NumVertices != Positions.Num())
					{
						UE_LOGF(LogChaosFleshGeneratorPrivate, Warning, "SkeletalMeshAsset and FleshAsset have different number of vertices in section %d. Check if the assets have the same mesh.", SectionIndex);
						return None;
					}

					for (int32 VertexIndex = 0; VertexIndex < MLDSection.NumVertices; ++VertexIndex)
					{
						const FVector3f& MLDPosition = MLDSection.SoftVertices[VertexIndex].Position;
						const FVector3f& FleshPosition = Positions[VertexIndex];
						if (!MLDPosition.Equals(FleshPosition, UE_KINDA_SMALL_NUMBER))
						{
							UE_LOGF(LogChaosFleshGeneratorPrivate, Warning, "SkeletalMeshAsset and FleshAsset have different vertex positions. Check if the assets have the same vertex order.");
							return None;
						}
					}
				}
			}

			return Map;
		}


	};

};

#undef LOCTEXT_NAMESPACE
