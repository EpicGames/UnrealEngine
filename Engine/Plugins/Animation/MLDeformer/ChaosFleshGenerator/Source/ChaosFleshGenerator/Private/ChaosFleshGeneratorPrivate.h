// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "GeometryCollection/ManagedArray.h"

class FFleshCollection;
class USkinnedAsset;
class UFleshAsset;
class UGeometryCache;
class USkeletalMesh;
class FSkeletalMeshLODRenderData;

DECLARE_LOG_CATEGORY_EXTERN(LogChaosFleshGeneratorPrivate, Log, All);


namespace UE::Chaos::FleshGenerator
{
	namespace Private
	{
		class FTimeScope
		{
			FString Name;
			FDateTime StartTime;
		public:
			explicit FTimeScope(FString InName);
			~FTimeScope();
		};

		TArray<int32> ParseFrames(const FString& FramesString);

		TArray<int32> Range(int32 End);

		TArray<uint32> Range(uint32 Start, uint32 End);

		TOptional<TArray<int32>> GetMeshImportVertexMap(const USkinnedAsset& SkinnedMeshAsset, const UFleshAsset& FleshAsset);
	};
};