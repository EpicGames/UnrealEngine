// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFleshGeneratorSimulation.h"

#include "BonePose.h"
#include "ChaosFlesh/FleshAsset.h"
#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFleshGeneratorPrivate.h"
#include "ChaosFleshGeneratorThreading.h"
#include "FleshGeneratorComponent.h"
#include "FleshGeneratorProperties.h"
#include "Engine/SkeletalMesh.h"
#include "Logging/LogMacros.h"
#include "ChaosFlesh/FleshCollectionEngineUtility.h"

DEFINE_LOG_CATEGORY(LogChaosFleshGeneratorSimulation);

#define LOCTEXT_NAMESPACE "ChaosFleshGeneratorSimulation"

namespace UE::Chaos::FleshGenerator
{
	FLaunchSimsTask::FLaunchSimsTask(TSharedPtr<FTaskResource> InTaskResource, TObjectPtr<UFleshGeneratorProperties> InProperties)
	: TaskResource(InTaskResource)
		, SimResources(InTaskResource->SimResources)
		, Properties(InProperties)
	{}

	void FLaunchSimsTask::DoWork()
	{	
		const int32 NumFrames = TaskResource->FramesToSimulate.Num();
		PrepareAnimationSequence();
	
		const int32 NumThreads = 1;
	
		for (int32 Frame = 0; Frame < NumFrames; Frame++)
		{
			if (!TaskResource->bCancelled.load())
			{
				const int32 AnimFrame = TaskResource->FramesToSimulate[Frame];
	
				const int32 ThreadIdx = 0;// Frame% NumThreads;
				FSimResource& SimResource = *SimResources[ThreadIdx].Get();
				Simulate(SimResource, AnimFrame, Frame);
			}
			else
			{
				break;
			}
		}

		RestoreAnimationSequence();
	}

	void FLaunchSimsTask::Simulate(FSimResource &SimResource, int32 AnimFrame, int32 CacheFrame) const
	{
		UFleshGeneratorComponent& FleshComponent = *SimResource.FleshComponent;
		USkeletalGeneratorComponent& SkeletalComponent = *SimResource.SkeletalComponent;
		UDeformableSolverComponent& SolverComponent = *SimResource.SolverComponent;
	
		const float DeltaTime = SolverComponent.SolverTiming.TimeStepSize;
		const ESaveType SaveType = ESaveType::LastStep;
	
		const TArray<FTransform> Transforms = GetBoneTransforms( SkeletalComponent, AnimFrame);
		FleshComponent.Pose(SkeletalComponent, Transforms);
		SolverComponent.WriteToSimulation(DeltaTime, false);
		SolverComponent.Simulate(DeltaTime);
		SolverComponent.ReadFromSimulation(DeltaTime,false);
		SimResource.SimulatedPositions[CacheFrame] = GetRenderPositions(SimResource);

		SimResource.FinishFrame();

	}
	
	void FLaunchSimsTask::PrepareAnimationSequence()
	{
		TObjectPtr<UAnimSequence> AnimationSequence = Properties->AnimationSequence;
		if (AnimationSequence)
		{
			InterpolationTypeBackup = AnimationSequence->Interpolation;
			AnimationSequence->Interpolation = EAnimInterpolationType::Step;
		}
	}
	
	void FLaunchSimsTask::RestoreAnimationSequence()
	{
		TObjectPtr<UAnimSequence> AnimationSequence = Properties->AnimationSequence;
		if (AnimationSequence)
		{
			AnimationSequence->Interpolation = InterpolationTypeBackup;
		}
	}
	
	TArray<FTransform> FLaunchSimsTask::GetBoneTransforms(USkeletalMeshComponent& InSkeletalComponent, int32 Frame) const
	{
		TArray<FTransform> ComponentSpaceTransforms;

		const UAnimSequence* AnimationSequence = Properties->AnimationSequence;
		const double Time = FMath::Clamp(AnimationSequence->GetSamplingFrameRate().AsSeconds(Frame), 0., (double)AnimationSequence->GetPlayLength());
		FAnimExtractContext ExtractionContext(Time);

		USkeletalMesh* SkeletalMesh = InSkeletalComponent.GetSkeletalMeshAsset();
		FReferenceSkeleton* ReferenceSkeleton = &SkeletalMesh->GetRefSkeleton();
		USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
		const int32 NumBones = ReferenceSkeleton ? ReferenceSkeleton->GetNum() : 0;
	
		TArray<uint16> BoneIndices;
		BoneIndices.SetNumUninitialized(NumBones);
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			BoneIndices[Index] = (uint16)Index;
		}
	
		FBoneContainer BoneContainer;
		BoneContainer.SetUseRAWData(true);
		BoneContainer.InitializeTo(BoneIndices, UE::Anim::FCurveFilterSettings(), *Skeleton);
	
		FCompactPose OutPose;
		OutPose.SetBoneContainer(&BoneContainer);
		FBlendedCurve OutCurve;
		OutCurve.InitFrom(BoneContainer);
		UE::Anim::FStackAttributeContainer TempAttributes;
	
		FAnimationPoseData AnimationPoseData(OutPose, OutCurve, TempAttributes);
		AnimationSequence->GetAnimationPose(AnimationPoseData, ExtractionContext);
	
		ComponentSpaceTransforms.SetNumUninitialized(NumBones);
		for (int32 Index = 0; Index < NumBones; ++Index)
		{
			const FCompactPoseBoneIndex CompactIndex = BoneContainer.MakeCompactPoseIndex(FMeshPoseBoneIndex(Index));
			const int32 ParentIndex = ReferenceSkeleton->GetParentIndex(Index);
			if (ComponentSpaceTransforms.IsValidIndex(ParentIndex) && ParentIndex < Index)
			{
				ComponentSpaceTransforms[Index] = AnimationPoseData.GetPose()[CompactIndex] * ComponentSpaceTransforms[ParentIndex];
			}
			else
			{
				ComponentSpaceTransforms[Index] = ReferenceSkeleton->GetRefBonePose()[Index];
			}
		}
		return ComponentSpaceTransforms;
		
	}
	
	TArray<FVector3f> FLaunchSimsTask::GetRenderPositions(FSimResource& SimResource) const
	{
		check(SimResource.FleshComponent);
		check(SimResource.SkeletalComponent);

		SimResource.SkeletalComponent->RecreateRenderState_Concurrent();

		TArray<FVector3f> Positions;
		const UFleshDynamicAsset* DynamicCollection = SimResource.FleshComponent->GetDynamicCollection();
		const UFleshAsset* RestCollection = SimResource.FleshComponent->GetRestCollection();
		const USkeletalMesh* SkeletalMesh = SimResource.SkeletalComponent->GetSkeletalMeshAsset();
		if (RestCollection && DynamicCollection && SkeletalMesh)
		{
			TSharedPtr<const FFleshCollection> FleshCollection = RestCollection->GetFleshCollection();
			const TManagedArray<FVector3f>* RestVertices = RestCollection->FindPositions();
			const TManagedArray<FVector3f>* SimulatedVertices = DynamicCollection->FindPositions();
			if (FleshCollection && RestVertices && SimulatedVertices)
			{
				ChaosFlesh::BoundSurfacePositions(SkeletalMesh, FleshCollection.Get(), RestVertices, SimulatedVertices, Positions);
			}
		}
		return Positions;
	}
};

#undef LOCTEXT_NAMESPACE
