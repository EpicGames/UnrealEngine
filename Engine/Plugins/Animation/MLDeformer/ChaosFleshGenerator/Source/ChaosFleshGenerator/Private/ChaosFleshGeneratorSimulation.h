// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Tasks/Pipe.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "Async/AsyncWork.h"
#include "FleshGeneratorComponent.h"

class UFleshGeneratorComponent;
class USkeletalGeneratorComponent;
class UDeformableSolverComponent;
class UFleshGeneratorProperties;
class FAsyncTaskNotification;
class UGeometryCache;

DECLARE_LOG_CATEGORY_EXTERN(LogChaosFleshGeneratorSimulation, Log, All);


namespace UE::Chaos::FleshGenerator
{
	struct FTaskResource;

	template<typename TaskType>
	class TTaskRunner : public FNonAbandonableTask
	{
	public:
		TTaskRunner(TUniquePtr<TaskType> InTask)
			: Task(MoveTemp(InTask))
		{
		}

		void DoWork()
		{
			if (Task)
			{
				Task->DoWork();
			}
		}

		bool CanAbandon()
		{
			return true;
		}

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(TTaskRunner, STATGROUP_ThreadPoolAsyncTasks);
		}

	private:
		TUniquePtr<TaskType> Task;
	};

	struct FSimResource
	{
		TObjectPtr<UFleshGeneratorComponent> FleshComponent = nullptr;
		TObjectPtr<USkeletalGeneratorComponent> SkeletalComponent = nullptr;
		TObjectPtr<UDeformableSolverComponent> SolverComponent = nullptr;

		TArrayView<TArray<FVector3f>> SimulatedPositions;
		std::atomic<int32>* NumSimulatedFrames = nullptr;

		std::atomic<bool>* bCancelled = nullptr;

		bool IsCancelled() const
		{
			return !bCancelled || bCancelled->load();
		}
		void FinishFrame()
		{
			if (NumSimulatedFrames)
			{
				++(*NumSimulatedFrames);
			}
		}
	};

	class FLaunchSimsTask
	{
	public:
		FLaunchSimsTask(TSharedPtr<FTaskResource> InTaskResource, TObjectPtr<UFleshGeneratorProperties> InProperties);

		void DoWork();

	private:
		using FPipe = ::UE::Tasks::FPipe;

		enum class ESaveType
		{
			LastStep,
			EveryStep,
		};

		void Simulate(FSimResource& SimResource, int32 AnimFrame, int32 CacheFrame) const;
		void PrepareAnimationSequence();
		void RestoreAnimationSequence();
		TArray<FTransform> GetBoneTransforms(USkeletalMeshComponent& InSkeletalComponent, int32 Frame) const;
		TArray<FVector3f> GetRenderPositions(FSimResource& SimResource) const;

		TSharedPtr<FTaskResource> TaskResource;
		TArray< TSharedPtr<FSimResource> >& SimResources;
		TObjectPtr<UFleshGeneratorProperties> Properties;
		EAnimInterpolationType InterpolationTypeBackup = EAnimInterpolationType::Linear;
	};




};