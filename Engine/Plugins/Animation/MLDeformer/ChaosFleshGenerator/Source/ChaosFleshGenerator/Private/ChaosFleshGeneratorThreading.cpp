// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosFleshGeneratorThreading.h"

#include "ChaosFlesh/FleshAsset.h"
#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "ChaosFleshGenerator.h"
#include "ChaosFleshGeneratorSimulation.h"
#include "Engine/World.h"
#include "FleshGeneratorComponent.h"
#include "FleshGeneratorProperties.h"
#include "GeometryCache.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY(LogChaosFleshGeneratorThreading);

#define LOCTEXT_NAMESPACE "ChaosFleshGeneratorThreading"

namespace UE::Chaos::FleshGenerator
{
	bool FTaskResource::AllocateSimResources_GameThread(TObjectPtr<UFleshGeneratorProperties> Properties, int32 Num)
	{
		UFleshAsset& Asset = *Properties->FleshAsset;
		if (!ensure(Num == 1)) return false;

		World = UWorld::CreateWorld(EWorldType::Editor, false);
		SimResources.SetNum(Num);
		for(int32 Index = 0; Index < Num; ++Index)
		{
			SimResources[Index] = TSharedPtr<FSimResource>(new FSimResource());
			AActor* Owner = World->SpawnActor<AActor>();

			UFleshGeneratorComponent* FleshComponent = NewObject<UFleshGeneratorComponent>(Owner);
			FleshComponent->SetRestCollection(&Asset);
			FleshComponent->RegisterComponentWithWorld(World);
	
			USkeletalGeneratorComponent* SkeletalMeshComponent = NewObject<USkeletalGeneratorComponent>(Owner);
			SkeletalMeshComponent->SetSkeletalMesh(Asset.SkeletalMesh); 
			SkeletalMeshComponent->RegisterComponentWithWorld(World);

			UDeformableSolverComponent* DeformableSolver = NewObject<UDeformableSolverComponent>(Owner);
			FleshComponent->EnableSimulation(DeformableSolver);
			DeformableSolver->RegisterComponentWithWorld(World);

			float TimeStepSize = (Properties->SolverTiming.FrameRate > 0) ? 1.0 / Properties->SolverTiming.FrameRate : 0;
			DeformableSolver->SolverTiming.FixTimeStep = true;
			DeformableSolver->SolverTiming.TimeStepSize = TimeStepSize;
			DeformableSolver->SolverTiming.NumSubSteps = FMath::Max(0, Properties->SolverTiming.NumSubSteps);
			DeformableSolver->SolverTiming.NumSolverIterations = FMath::Max(0, Properties->SolverTiming.NumIterations);
			DeformableSolver->SolverEvolution = Properties->SolverEvolution;
			DeformableSolver->SolverCollisions = Properties->SolverCollisions;
			DeformableSolver->SolverConstraints = Properties->SolverConstraints;
			DeformableSolver->SolverForces = Properties->SolverForces;
			DeformableSolver->SolverDebugging = Properties->SolverDebugging;
			DeformableSolver->BuildSimulationProxy();


			constexpr int32 LODIndex = 0;
			SkeletalMeshComponent->SetForcedLOD(LODIndex + 1);
			SkeletalMeshComponent->UpdateLODStatus();
			SkeletalMeshComponent->RefreshBoneTransforms(nullptr);

			SkeletalMeshComponent->bRenderStatic = false;
			constexpr bool bRecreateRenderStateImmediately = true;
			SkeletalMeshComponent->SetCPUSkinningEnabled(true, bRecreateRenderStateImmediately);

			FSimResource& SimResource = *SimResources[Index];
			SimResource.FleshComponent = TObjectPtr<UFleshGeneratorComponent>(FleshComponent);
			SimResource.SkeletalComponent = TObjectPtr<USkeletalGeneratorComponent>(SkeletalMeshComponent);
			SimResource.SolverComponent = TObjectPtr<UDeformableSolverComponent>(DeformableSolver);
				
			SimResource.SimulatedPositions = TArrayView<TArray<FVector3f>>(SimulatedPositions);
			SimResource.NumSimulatedFrames = &NumSimulatedFrames;
			SimResource.bCancelled = &bCancelled;
		}

		return true;
	}
	
	void FTaskResource::FreeSimResources_GameThread()
	{
		if (Executer.IsValid())
		{
			Executer->EnsureCompletion();
		}
		for (TSharedPtr<FSimResource>& SimResource : SimResources)
		{
			SimResource->FleshComponent->UnregisterComponent();
			SimResource->FleshComponent->DestroyComponent();
		}
		SimResources.Empty();
		World->DestroyWorld(false);
	}
	
	void FTaskResource::FlushRendering()
	{
	}

	void FTaskResource::Cancel()
	{
		bCancelled.store(true);
		Executer->TryAbandonTask();
	}
};

#undef LOCTEXT_NAMESPACE
