// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "ChaosFleshGeneratorSimulation.h"

//#include "Async/AsyncWork.h"
//#include "TickableEditorObject.h"
//#include "UObject/StrongObjectPtr.h"

class FAsyncTaskNotification;
class UGeometryCache;
struct FSimResource;

DECLARE_LOG_CATEGORY_EXTERN(LogChaosFleshGeneratorThreading, Log, All);


namespace UE::Chaos::FleshGenerator
{
	struct FTaskResource
	{
		using FExecuterType = FAsyncTask<TTaskRunner<FLaunchSimsTask>>;

		TArray<TSharedPtr<FSimResource>> SimResources;

		TUniquePtr<FExecuterType> Executer;
		TUniquePtr<FAsyncTaskNotification> Notification;
		FDateTime StartTime;
		FDateTime LastUpdateTime;

		TArray<int32> FramesToSimulate;
		TArray<TArray<FVector3f>> SimulatedPositions;
		TArray<uint32> ImportedVertexNumbers;
		UGeometryCache* Cache = nullptr;

		std::atomic<int32> NumSimulatedFrames = 0;
		std::atomic<bool> bCancelled = false;

		UWorld* World = nullptr;

		bool AllocateSimResources_GameThread(TObjectPtr<UFleshGeneratorProperties> Properties, int32 Num);
		void FreeSimResources_GameThread();
		void FlushRendering();
		void Cancel();
	};

};