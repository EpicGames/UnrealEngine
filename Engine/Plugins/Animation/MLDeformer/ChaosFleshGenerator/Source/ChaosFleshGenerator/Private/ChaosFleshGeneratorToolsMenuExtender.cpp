// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosFleshGeneratorToolsMenuExtender.h"

#include "FleshGeneratorProperties.h"
#include "Engine/SkeletalMesh.h"
#include "Framework/Docking/TabManager.h"
#include "IDocumentation.h"
#include "MLDeformerGeomCacheEditorModel.h"
#include "MLDeformerEditorModel.h"
#include "MLDeformerModel.h"
#include "MLDeformerGeomCacheModel.h"
#include "SFleshGeneratorWidget.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "ChaosFleshGeneratorToolsMenuExtender"

namespace UE::Chaos::FleshGenerator
{
	namespace Private
	{
		void SpawnTab(UE::MLDeformer::FMLDeformerEditorToolkit& Toolkit)
		{
			TSharedPtr<SDockTab> Tab = Toolkit.GetAssociatedTabManager()->TryInvokeTab(FName(FChaosFleshGeneratorTabSummoner::TabID));
			if (!Tab.IsValid())
			{
				return;
			}
			
			TSharedRef<SWidget> TabContent = Tab->GetContent();
			if (TabContent == SNullWidget::NullWidget || TabContent->GetTypeAsString() != "SFleshGeneratorWidget")
			{
				return;
			}
			TSharedRef<SFleshGeneratorWidget> Widget = StaticCastSharedRef<SFleshGeneratorWidget>(Tab->GetContent());
			if (Widget == SNullWidget::NullWidget)
			{
				return;
			}
		
			TWeakObjectPtr<UFleshGeneratorProperties> Properties = Widget->GetProperties();
			if (!Properties.IsValid())
			{
				return;
			}

			using UE::MLDeformer::FMLDeformerEditorModel;
			if (FMLDeformerEditorModel* EditorModel = Toolkit.GetActiveModel())
			{
				if (UMLDeformerModel* Model = EditorModel->GetModel())
				{
					Properties->SkeletalMeshAsset = Model->GetSkeletalMesh();
					Properties->AnimationSequence = EditorModel->GetActiveTrainingInputAnimSequence();
					if (UMLDeformerGeomCacheModel* GeomCacheModel = Cast<UMLDeformerGeomCacheModel>(Model))
					{
						const UE::MLDeformer::FMLDeformerGeomCacheEditorModel* GeomCacheEditorModel = static_cast<const UE::MLDeformer::FMLDeformerGeomCacheEditorModel*>(EditorModel);
						if (UGeometryCache* GeomCache = GeomCacheEditorModel->GetActiveGeometryCache())
						{
							Properties->SimulatedCache = GeomCache;
						}
					}
				}
			}
		}
	};
	
	TUniquePtr<FChaosFleshGeneratorToolsMenuExtender> CreateToolsMenuExtender()
	{
		return MakeUnique<FChaosFleshGeneratorToolsMenuExtender>();
	}
	
	const FName FChaosFleshGeneratorTabSummoner::TabID = FName("ChaosFleshGenerator");
	
	FChaosFleshGeneratorTabSummoner::FChaosFleshGeneratorTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& InEditor)
		: FWorkflowTabFactory(TabID, InEditor)
	{
		bIsSingleton = true;
		TabLabel = LOCTEXT("ChaosFleshGenerator", "Chaos Flesh Generator");
		ViewMenuDescription = LOCTEXT("ViewMenu_Desc", "Chaos Flesh Generator");
		ViewMenuTooltip = LOCTEXT("ViewMenu_ToolTip", "Show the Chaos Flesh Generator tab.");
	}
	
	TSharedRef<SWidget> FChaosFleshGeneratorTabSummoner::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
	{
		return SNew(SFleshGeneratorWidget);
	}
	
	TSharedPtr<SToolTip> FChaosFleshGeneratorTabSummoner::CreateTabToolTipWidget(const FWorkflowTabSpawnInfo& Info) const
	{
		return IDocumentation::Get()->CreateToolTip(
			LOCTEXT("ChaosFleshGeneratorWidgetTooltip", "Generate training data using chaos cloth solver."),
			NULL,
			TEXT("Shared/Editors/Persona"),
			TEXT("ChaosFleshGenerator_Window"));
	}
	
	FMenuEntryParams FChaosFleshGeneratorToolsMenuExtender::GetMenuEntry(FMLDeformerEditorToolkit& Toolkit) const
	{
		FMenuEntryParams Params;
		Params.DirectActions = FUIAction(
			FExecuteAction::CreateLambda([&Toolkit]()
			{
				Private::SpawnTab(Toolkit);
			}),
			FCanExecuteAction::CreateLambda([]()
			{
				return true;
			})
		);
		Params.LabelOverride = LOCTEXT("ChaosFleshGenerator", "Chaos Flesh Generator");
		Params.ToolTipOverride = LOCTEXT("ChaosFleshGeneratorMenuTooltip", "Generate training data using chaos cloth solver");
		return Params;
	}
	
	TSharedPtr<FWorkflowTabFactory> FChaosFleshGeneratorToolsMenuExtender::GetTabSummoner(const TSharedRef<FMLDeformerEditorToolkit>& Toolkit) const
	{
		return MakeShared<FChaosFleshGeneratorTabSummoner>(Toolkit);
	}
};

#undef LOCTEXT_NAMESPACE