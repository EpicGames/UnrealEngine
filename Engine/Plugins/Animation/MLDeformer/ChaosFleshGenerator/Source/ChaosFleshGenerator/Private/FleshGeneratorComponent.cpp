// Copyright Epic Games, Inc. All Rights Reserved.

#include "FleshGeneratorComponent.h"


//
// UFleshGeneratorComponent
//


UFleshGeneratorComponent::UFleshGeneratorComponent(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
{
}

UFleshGeneratorComponent::~UFleshGeneratorComponent() = default;

void UFleshGeneratorComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
    UActorComponent::TickComponent(DeltaTime, TickType, ThisTickFunction);
}

void UFleshGeneratorComponent::Pose(USkeletalGeneratorComponent& InSkeletalComponent, const TArray<FTransform>& InComponentSpaceTransforms)
{
	if (!ensure(InComponentSpaceTransforms.Num() == InSkeletalComponent.GetComponentSpaceTransforms().Num()))
	{
		return;
	}
	InSkeletalComponent.GetEditableComponentSpaceTransforms() = InComponentSpaceTransforms;
	InSkeletalComponent.FlipSpaceBuffer();
	InSkeletalComponent.FinalizeBoneTransform();

	InSkeletalComponent.UpdateBounds();
    if (IsInGameThread())
    {
		InSkeletalComponent.MarkRenderTransformDirty();
		InSkeletalComponent.MarkRenderDynamicDataDirty();
    }
}

//
// USkeletalGeneratorComponent
//

USkeletalGeneratorComponent::USkeletalGeneratorComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

USkeletalGeneratorComponent::~USkeletalGeneratorComponent() = default;

void USkeletalGeneratorComponent::FlipSpaceBuffer()
{
	bNeedToFlipSpaceBaseBuffers = true;

}
