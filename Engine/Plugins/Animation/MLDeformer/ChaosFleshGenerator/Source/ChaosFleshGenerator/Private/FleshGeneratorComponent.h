// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Deformable/ChaosDeformableSolverProxy.h"
#include "ChaosFlesh/FleshComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "ChaosFlesh/ChaosDeformableTetrahedralComponent.h"


#include "FleshGeneratorComponent.generated.h"

class USkeletalGeneratorComponent;


/**
 * Flesh data generation component.
 */
UCLASS()
class UFleshGeneratorComponent : public UFleshComponent
{
	GENERATED_BODY()	
public:
	UFleshGeneratorComponent(const FObjectInitializer& ObjectInitializer);
	~UFleshGeneratorComponent();

	/** Pose the cloth component using component space transforms. */
    void Pose(USkeletalGeneratorComponent& InSkeletalComponent, const TArray<FTransform>& InComponentSpaceTransforms);
protected:
	//~ Begin UActorComponent Interface
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	//~ End UActorComponent Interface
};


/**
 * USkeletalGeneratorComponent data generation component.
 */
UCLASS()
class USkeletalGeneratorComponent : public USkeletalMeshComponent
{
	GENERATED_BODY()
public:
	USkeletalGeneratorComponent(const FObjectInitializer& ObjectInitializer);
	~USkeletalGeneratorComponent();
	void FlipSpaceBuffer();
};
