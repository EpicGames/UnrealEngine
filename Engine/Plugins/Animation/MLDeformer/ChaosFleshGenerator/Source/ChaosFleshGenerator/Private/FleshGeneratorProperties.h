// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ChaosFlesh/FleshAsset.h"
#include "ChaosFlesh/ChaosDeformableSolverComponent.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkinnedAsset.h"
#include "GeometryCache.h"

#include "FleshGeneratorProperties.generated.h"


USTRUCT(BlueprintType)
struct FFleshGeneratorSolverTimingGroup
{
	GENERATED_USTRUCT_BODY()

		/* Frame rate of the training animation (FrameDeltaTime = 1/FrameRate) */
		UPROPERTY(EditAnywhere, Category = "SolverTiming", meta = (Min = 1))
		float FrameRate = 24.;

	/* Number of frames to simulate ( the length of the animation) */
	UPROPERTY(EditAnywhere, Category = "SolverTiming", meta = (Min = 0))
		int32 NumFrames = 150;

	/* Number of subdivisions within a time step (SolverStep = FrameDeltaTime/NumSubSteps) */
	UPROPERTY(EditAnywhere, Category = "SolverTiming", meta = (Min = 0))
		int32 NumSubSteps = 2;

	/* Number of convergence steps for the constraint solver for each solver step. */
	UPROPERTY(EditAnywhere, Category = "SolverTiming", meta = (Min = 0))
		int32 NumIterations = 5;

};


UCLASS()
class UFleshGeneratorProperties : public UObject
{
	GENERATED_BODY()
public:
	/* Skeletal mesh that will be used in MLDeformer */
	UPROPERTY(EditAnywhere, Category = Input)
	TObjectPtr<USkinnedAsset> SkeletalMeshAsset;

	/* Chaos cloth asset used in simulation. This should be different from the skeletal mesh asset. */
	UPROPERTY(EditAnywhere, Category = Input)
	TObjectPtr<UFleshAsset> FleshAsset;

	/* Training poses. */
	UPROPERTY(EditAnywhere, Category = Input)
	TObjectPtr<UAnimSequence> AnimationSequence;

	/* e.g. "0, 2, 5-10, 12-15". If left empty, all frames will be used */
	UPROPERTY(EditAnywhere, Category = Input)
	FString FramesToSimulate;

	/* Output meshes */
	UPROPERTY(EditAnywhere, Category = Output)
	TObjectPtr<UGeometryCache> SimulatedCache;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "SimulationSettings")
	FFleshGeneratorSolverTimingGroup SolverTiming;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "SimulationSettings")
	FSolverEvolutionGroup SolverEvolution;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "SimulationSettings")
	FSolverCollisionsGroup SolverCollisions;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "SimulationSettings")
	FSolverConstraintsGroup SolverConstraints;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "SimulationSettings")
	FSolverForcesGroup SolverForces;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "SimulationSettings")
	FSolverDebuggingGroup SolverDebugging;
};