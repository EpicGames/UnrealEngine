// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Widgets/SCompoundWidget.h"

class UFleshGeneratorProperties;
class IDetailsView;
namespace UE::Chaos::FleshGenerator
{
	class FChaosFleshGenerator;

	class SFleshGeneratorWidget : public SCompoundWidget
	{
		SLATE_BEGIN_ARGS(SFleshGeneratorWidget) 
		{}
		SLATE_END_ARGS()

	public:
		void Construct(const FArguments& InArgs);
		TWeakObjectPtr<UFleshGeneratorProperties> GetProperties() const;

	private:
		TSharedPtr<IDetailsView> DetailsView;
		TSharedPtr<FChaosFleshGenerator> ChaosFleshGenerator;
	};

	class FFleshGeneratorDetails : public IDetailCustomization
	{
	public:
		static TSharedRef<IDetailCustomization> MakeInstance();
		virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	private:
		TWeakObjectPtr<UFleshGeneratorProperties> GetProperties(IDetailLayoutBuilder& DetailBuilder) const;
	};
};